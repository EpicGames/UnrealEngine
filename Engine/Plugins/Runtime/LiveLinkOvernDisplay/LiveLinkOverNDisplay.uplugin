{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Link Over nDisplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "LiveLink subjects synchronization for nDisplay setup",
	"Category": "Misc",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64", "Linux" ],
	"Modules": [
		{
			"Name": "LiveLinkOverNDisplay",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "nDisplay",
			"Enabled": true
		}
	]
}
