// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkOverNDisplay : ModuleRules
{
	public LiveLinkOverNDisplay(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DisplayCluster",
				"Engine",
				"LiveLinkInterface",
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Settings"
				});
		}
	}
}
