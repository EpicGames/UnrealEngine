// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOverNDisplayModule.h"

#include "DisplayClusterEnums.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "LiveLinkOverNDisplayPrivate.h"
#include "LiveLinkOverNDisplaySettings.h"
#include "Misc/CoreDelegates.h"
#include "NDisplayLiveLinkSubjectReplicator.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
#endif 


DEFINE_LOG_CATEGORY(LogLiveLinkOverNDisplay);

#define LOCTEXT_NAMESPACE "LiveLinkOverNDisplayModule"


FLiveLinkOverNDisplayModule::FLiveLinkOverNDisplayModule()
	: LiveLinkReplicator(MakeUnique<FNDisplayLiveLinkSubjectReplicator>())
{

}

void FLiveLinkOverNDisplayModule::StartupModule()
{
	// Register Engine callbacks
	FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FLiveLinkOverNDisplayModule::OnEngineLoopInitComplete);

#if WITH_EDITOR
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterSettings(
			"Project", "Plugins", "LiveLink over nDisplay",
			LOCTEXT("LiveLinkOverNDisplaySettingsName", "LiveLink over nDisplay"),
			LOCTEXT("LiveLinkOverNDisplaySettingsDescription", "Configure LiveLink over nDisplay."),
			GetMutableDefault<ULiveLinkOverNDisplaySettings>()
		);
	}
#endif
}

void FLiveLinkOverNDisplayModule::ShutdownModule()
{
#if WITH_EDITOR
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "LiveLink over nDisplay");
	}
#endif

	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);
}

FNDisplayLiveLinkSubjectReplicator& FLiveLinkOverNDisplayModule::GetSubjectReplicator()
{
	return *LiveLinkReplicator;
}

void FLiveLinkOverNDisplayModule::OnEngineLoopInitComplete()
{
	//Register to display cluster scene event delegates to know when to activate/deactivate replicator. 
	//i.e. When loading a map, scene will be ended and all SyncObjects will be destroyed.
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartScene().AddRaw(this, &FLiveLinkOverNDisplayModule::OnDisplayClusterStartSceneCallback);
	IDisplayCluster::Get().GetCallbacks().OnDisplayClusterEndScene().AddRaw(this, &FLiveLinkOverNDisplayModule::OnDisplayClusterEndSceneCallback);

	//Initialize replicator so it's ready to operate
	LiveLinkReplicator->Initialize();

	//Start scene event has already been done when engine has finished initializing so fake it
	OnDisplayClusterStartSceneCallback();
}

void FLiveLinkOverNDisplayModule::OnDisplayClusterStartSceneCallback() const
{
	if (IDisplayCluster::IsAvailable() && IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster) 
	{
		if (GetDefault<ULiveLinkOverNDisplaySettings>()->IsLiveLinkOverNDisplayEnabled())
		{
			//When Scene is started, activate our replicator sync object ( registers itself )
			LiveLinkReplicator->Activate();
		}
	}
}

void FLiveLinkOverNDisplayModule::OnDisplayClusterEndSceneCallback() const
{
	//When scene is ended, deactivate replicator sync object ( unregisters itself )
	LiveLinkReplicator->Deactivate();
}

#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FLiveLinkOverNDisplayModule, LiveLinkOverNDisplay);

