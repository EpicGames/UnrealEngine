// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ILiveLinkOverNDisplayModule.h"


class FNDisplayLiveLinkSubjectReplicator;


/**
 * Entry point for LiveLinkOverNDisplay functionality. 
 */
class FLiveLinkOverNDisplayModule : public ILiveLinkOverNDisplayModule, public TSharedFromThis<FLiveLinkOverNDisplayModule>
{
public:

	FLiveLinkOverNDisplayModule();

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	virtual FNDisplayLiveLinkSubjectReplicator& GetSubjectReplicator() override;


private:

	void OnEngineLoopInitComplete();
	void OnDisplayClusterStartSceneCallback() const;
	void OnDisplayClusterEndSceneCallback() const;

private:

	/** Replicator SyncObject used to transfer data across all cluster machines */
	TUniquePtr<FNDisplayLiveLinkSubjectReplicator> LiveLinkReplicator;
};
