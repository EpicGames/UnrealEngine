// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOverNDisplaySettings.h"

#include "LiveLinkOverNDisplayPrivate.h"
#include "Misc/CommandLine.h"


ULiveLinkOverNDisplaySettings::ULiveLinkOverNDisplaySettings()
{
	FString BoolValue;
	const bool bIsFoundOnCommandLine = FParse::Value(FCommandLine::Get(), TEXT("-EnableLiveLinkOverNDisplay="), BoolValue);

	if (bIsFoundOnCommandLine)
	{
		UE_LOGF(LogLiveLinkOverNDisplay, Log, "Overriding LiveLinkOverNDisplay enable flag from command line with value '%ls'", *BoolValue);
		bIsEnabledFromCommandLine = BoolValue.ToBool();
	}
}
