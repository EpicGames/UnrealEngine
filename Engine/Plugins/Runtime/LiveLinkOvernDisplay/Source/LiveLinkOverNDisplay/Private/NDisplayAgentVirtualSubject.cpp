// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDisplayAgentVirtualSubject.h"




void UNDisplayAgentVirtualSubject::Update()
{
	UpdateTranslatorsForThisFrame();
}

void UNDisplayAgentVirtualSubject::UpdateFrameData(FLiveLinkFrameDataStruct&& NewFrameData)
{
	UpdateFrameDataSnapshot(MoveTemp(NewFrameData));
}

void UNDisplayAgentVirtualSubject::SetTrackedSubjectInfo(const FLiveLinkSubjectKey& InSubjectKey, TSubclassOf<ULiveLinkRole> InRole)
{
	AssociatedSubject = InSubjectKey;
	Role = InRole;
}

void UNDisplayAgentVirtualSubject::UpdateTranslators(const TArray<ULiveLinkFrameTranslator*>& SourceTranslators)
{
	for (const ULiveLinkFrameTranslator* SourceTranslator : SourceTranslators)
	{
		ULiveLinkFrameTranslator* NewTranslator = DuplicateObject(SourceTranslator, this);
		FrameTranslators.Add(NewTranslator);
	}
}
