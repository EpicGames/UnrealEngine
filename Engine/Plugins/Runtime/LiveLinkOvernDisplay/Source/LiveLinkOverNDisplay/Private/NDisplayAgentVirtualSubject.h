// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkVirtualSubject.h"


#include "NDisplayAgentVirtualSubject.generated.h"

class ULiveLinkFrameTranslator;


/**
 * LiveLink VirtualSubject used on nDisplay agents to replicate real subjects
 * Controller sends data to use for each frame to make sure all machines display / uses the same data
 * Uses Replicator object that this module creates.
 */
UCLASS()
class UNDisplayAgentVirtualSubject : public ULiveLinkVirtualSubject
{
	GENERATED_BODY()

public:
	
	//~ Begin ULiveLinkVirtualSubject interface
	virtual void Update() override;
	//~ End ULiveLinkVirtualSubject interface

	void UpdateFrameData(FLiveLinkFrameDataStruct&& NewFrameData);


	void SetTrackedSubjectInfo(const FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole> Role);

	void UpdateTranslators(const TArray<ULiveLinkFrameTranslator*>& SourceTranslators);

	FLiveLinkSubjectKey GetAssociatedSubjectKey() const { return AssociatedSubject; }

private:

	/** Real subject that this virtual subject replicates */
	FLiveLinkSubjectKey AssociatedSubject;
};
