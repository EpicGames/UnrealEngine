// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDisplayLiveLinkSubjectReplicator.h"


#include "Cluster/IDisplayClusterClusterManager.h"
#include "DisplayClusterEnums.h"
#include "Features/IModularFeatures.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "ILiveLinkClient.h"
#include "LiveLinkRole.h"
#include "LiveLinkSubjectSettings.h"
#include "Misc/CoreDelegates.h"
#include "NDisplayAgentVirtualSubject.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/Package.h"


DEFINE_LOG_CATEGORY_STATIC(LogNDisplayLiveLinkSubjectReplicator, Log, All);


FNDisplayLiveLinkSubjectReplicator::~FNDisplayLiveLinkSubjectReplicator()
{
	Deactivate();
	Release();
}

bool FNDisplayLiveLinkSubjectReplicator::IsActive() const
{
	return LiveLinkClient != nullptr;
}

FString FNDisplayLiveLinkSubjectReplicator::GetSyncId() const
{
	static const FString SyncId = TEXT("NDisplayLiveLinkSyncObject");
	return SyncId;
}

void FNDisplayLiveLinkSubjectReplicator::SerializeDC(FArchive& Ar)
{
	// Don't deserialize on the primary nodes as those are basically the data sources.
	// Moreover, the underlying logic is not supposed to be called there.
	if (Ar.IsLoading())
	{
		if (IDisplayCluster::Get().GetClusterMgr()->IsPrimary())
		{
			return;
		}
	}

	// Serialize/deserialize internally
	OnDataSynchronization(Ar);
}

void FNDisplayLiveLinkSubjectReplicator::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& Subject : TrackedSubjects)
	{
		Collector.AddReferencedObject(Subject);
	}
}

void FNDisplayLiveLinkSubjectReplicator::Initialize()
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		//Cache LiveLink client if it was available and listen for it being unregistered.
		LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		IModularFeatures::Get().OnModularFeatureUnregistered().AddRaw(this, &FNDisplayLiveLinkSubjectReplicator::OnModularFeatureRemoved);

		if (IDisplayCluster::IsAvailable())
		{
			if (IDisplayCluster::Get().GetClusterMgr()->IsSecondary())
			{
				IDisplayCluster::Get().GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().AddRaw(this, &FNDisplayLiveLinkSubjectReplicator::OnDisplayClusterPrimaryNodeChanged);

				//If we're a agent, listen for new subject and disables them if we're tracking that subject
				FCoreDelegates::OnBeginFrame.AddRaw(this, &FNDisplayLiveLinkSubjectReplicator::OnEngineBeginFrame);

				//Used to reinitialize ourself if we are removed from livelink
				LiveLinkClient->OnLiveLinkSourceRemoved().AddRaw(this, &FNDisplayLiveLinkSubjectReplicator::OnLiveLinkSourceRemoved);

				ReInitializeVirtualSource();
			}
		}
		else
		{
			UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Error, "Can't initialize LiveLink Subject Replicator for nDisplay because nDisplay is not available.");
		}
	}
	else
	{
		UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Error, "Can't initialize LiveLink Subject Replicator for nDisplay because LiveLink is not available.");
	}
}

void FNDisplayLiveLinkSubjectReplicator::Release()
{
	IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);
	FCoreDelegates::OnBeginFrame.RemoveAll(this);

	if (LiveLinkClient && IModularFeatures::Get().IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		LiveLinkClient->OnLiveLinkSourceRemoved().RemoveAll(this);
		LiveLinkClient->RemoveSource(LiveLinkSourceGuid);
	}

	if (IDisplayCluster::IsAvailable())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterFailoverPrimaryNodeChanged().RemoveAll(this);
	}

	//Cleanup our virtual source from LiveLink
	TrackedSubjects.Empty();
}

void FNDisplayLiveLinkSubjectReplicator::Activate()
{
	UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Log, "Registering sync object.");

	//Register SyncObject on PreTick to have this behavior

	//Order of operation will be
	//1. When LiveLink creates snapshots for the frame, Controller will create the sync object payload
	//2. On PreTick, the SyncObject will be synchronized between cluster machines
	//3. On object processing, Agents will push received data for each subject to have the same data then Controller
	if (IDisplayCluster::IsAvailable())
	{
		IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
		ClusterManager->RegisterSyncObject(this, EDisplayClusterSyncGroup::PreTick);
	}
	else
	{
		UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Warning, "Can't activate LiveLink Subject Replicator for nDisplay because nDisplay is not available.");
	}
}

void FNDisplayLiveLinkSubjectReplicator::Deactivate()
{
	UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Log, "Unregistering sync object.");
	
	if (IDisplayCluster::IsAvailable())
	{
		if (IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
		{
			ClusterManager->UnregisterSyncObject(this);
		}
	}
}

void FNDisplayLiveLinkSubjectReplicator::OnEngineBeginFrame()
{
	if (bBecamePrimary)
	{
		// It's not an agent anymore. Now it's a controller, therefore re-initialize.
		Release();
		Initialize();

		// Since we're primary now, we need to enable all LL subjects. They were intentionally
		// disabled when we this node was an agent (non-primary).
		constexpr bool bIncludeDisabled = true; // including disabled ones
		constexpr bool bIncludeVirtual = true;
		SetSubjectsEnabled(bIncludeDisabled, bIncludeVirtual, true);

		bBecamePrimary = false;
	}
	//Each frame, disable all subjects. When we are enabled, we are replicating ALL subjects from the controller to each agent.
	//Only our replicated VirtualSubjects should be enabled
	else if (LiveLinkClient)
	{
		check(IDisplayCluster::Get().GetClusterMgr()->IsSecondary());

		const bool bIncludeDisabled = false;
		const bool bIncludeVirtual = true;
		SetSubjectsEnabled(bIncludeDisabled, bIncludeVirtual, false);

		//Make sure all our replicated subjects are enabled
		for (const UNDisplayAgentVirtualSubject* OurSubject : TrackedSubjects)
		{
			LiveLinkClient->SetSubjectEnabled(OurSubject->GetSubjectKey(), true);
		}
	}
}

void FNDisplayLiveLinkSubjectReplicator::OnDataSynchronization(FArchive& Ar)
{
	//Controller is saving, agents are loading
	const bool bIsSaving = Ar.IsSaving();
	bool bContinue = bIsSaving;

	EFrameType FrameType = EFrameType::DataOnly;

	//Fill in current tracked subject list. If a subject is not processed this frame, we remove it
	TArray<FLiveLinkSubjectKey, TInlineAllocator<16>> LastFrameSubjectKeys;
	for (const UNDisplayAgentVirtualSubject* Subject : TrackedSubjects)
	{
		LastFrameSubjectKeys.Add(Subject->GetAssociatedSubjectKey());
	}

	if (bIsSaving)
	{
		if (LiveLinkClient)
		{
			/**
			 * We're the controller, and serializing data out to clients:
			 *	- Iterate over all of the snapshots.
			 *		- Determine the type of frame this is (new static data, new subject, data only).
			 *		- Serialize a True continuation byte.
			 *		- Serialize the frame.
			 *	- Once all snapshots have been added, serialize a False continuation byte.
			 */
			{
				//Get all subjects except the ones that are disabled
				//Since we are replicating using a VirtualSubject, agents will have to remove any VirtualSubject being replicated
				const bool bIncludeDisabled = false;
				const bool bIncludeVirtuals = true;
				TArray<FLiveLinkSubjectKey> ThisFramesSubjects = LiveLinkClient->GetSubjects(bIncludeDisabled, bIncludeVirtuals);
				for (FLiveLinkSubjectKey& SubjectKey : ThisFramesSubjects)
				{
					const FName SubjectName = SubjectKey.SubjectName;
					TSubclassOf<ULiveLinkRole> SubjectRole = LiveLinkClient->GetSubjectRole_AnyThread(SubjectKey);

					// Check if the subject's source is pending kill. If it is, the subject itself will soon be killed and we can skip evaluating it.

					TArray<FGuid> ValidSources = LiveLinkClient->GetSources(false);

					if (bIncludeVirtuals)
					{
						ValidSources += LiveLinkClient->GetVirtualSources(false);
					}

					const bool bIsSourceValid = ValidSources.ContainsByPredicate([SubjectKey](const FGuid& Other) { return Other == SubjectKey.Source; });

					if (!bIsSourceValid)
					{
						UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "Controller could not evaluate Subject '%ls'. Its source is marked as pending kill.", *SubjectName.ToString());
						continue;
					}

					FLiveLinkSubjectFrameData SubjectFrameData;
					if (LiveLinkClient->EvaluateFrame_AnyThread(SubjectName, SubjectRole, SubjectFrameData))
					{
						if (auto* FoundSujectDataPtr = TrackedSubjects.FindByPredicate(
							[SubjectKey](const UNDisplayAgentVirtualSubject* Other) { return Other->GetAssociatedSubjectKey().SubjectName == SubjectKey.SubjectName; }))
						{
							UNDisplayAgentVirtualSubject* FoundSubjectData = *FoundSujectDataPtr;
							if (SubjectFrameData.StaticData != FoundSubjectData->GetStaticData() 
								|| FoundSubjectData->GetAssociatedSubjectKey() != SubjectKey 
								|| FoundSubjectData->GetRole() != SubjectRole)
							{
								FrameType = EFrameType::UpdatedSubject;
							}
							else
							{
								FrameType = EFrameType::DataOnly;
							}
						}
						else
						{
							FrameType = EFrameType::NewSubject;
						}

						//This subject was processed
						LastFrameSubjectKeys.Remove(SubjectKey);

						Ar << bContinue;

						HandleFrame(Ar, FrameType, SubjectKey, SubjectRole, SubjectFrameData);
					}
					else
					{
						UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "Controller could not evaluate Subject '%ls'. It could have starved.", *SubjectName.ToString());
					}
				}
			}
		}

		//We're done processing subjects, mark the frame as done 
		bContinue = false;
		Ar << bContinue;
	}
	else
	{
		/**
		 * We're a agent, and serializing data from the controller:
		 *	- Serialize the continuation byte
		 *		- If False, we're done.
		 *		- If True:
		 *			- Serialize the frame.
		 *			- Based on frame type, do required process.
		 *			- Repeat.
		 */

		TSubclassOf<ULiveLinkRole> SubjectRole;
		FLiveLinkSubjectKey SubjectKey;
		FLiveLinkSubjectFrameData SubjectFrameData;

		while (true)
		{
			Ar << bContinue;
			if (bContinue)
			{
				//Process frame data and fill this subject's data
				HandleFrame(Ar, FrameType, SubjectKey, SubjectRole, SubjectFrameData);
				if (!SubjectKey.SubjectName.IsNone())
				{
					ProcessLiveLinkData_Agent(FrameType, SubjectKey, SubjectFrameData.FrameData);

					LastFrameSubjectKeys.RemoveSingleSwap(SubjectKey);
				}
			}
			else
			{
				break;
			}
		}
	}

	// Cleanup missing subjects. Agents will need to remove VirtualSubjects that were created
	if (LiveLinkClient)
	{
		for (const FLiveLinkSubjectKey& ItemSubjectKey : LastFrameSubjectKeys)
		{
			UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "Subject '%ls' was not processed. Removing it from our list.", *ItemSubjectKey.SubjectName.ToString());

			if (!bIsSaving)
			{
				const FLiveLinkSubjectKey ReplicatedSubjectKey(LiveLinkSourceGuid, ItemSubjectKey.SubjectName);
				LiveLinkClient->RemoveVirtualSubject(ReplicatedSubjectKey);
			}

			const int32 FoundIndex = TrackedSubjects.IndexOfByPredicate([ItemSubjectKey](const UNDisplayAgentVirtualSubject* Other) { return Other->GetAssociatedSubjectKey() == ItemSubjectKey; });
			if (TrackedSubjects.IsValidIndex(FoundIndex))
			{
				TrackedSubjects.RemoveAtSwap(FoundIndex);
			}
		}
	}
	else
	{
		TrackedSubjects.Reset();
	}
}

void FNDisplayLiveLinkSubjectReplicator::HandleFrame(FArchive& Ar, EFrameType& FrameType, FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole>& SubjectRole, FLiveLinkSubjectFrameData& SubjectFrame)
{
	//Archive always start with the type and subject identifier
	Ar << FrameType;
	Ar << SubjectKey;

	//If it's a new subject / updated one, process static data and role
	if (FrameType == EFrameType::NewSubject || FrameType == EFrameType::UpdatedSubject)
	{
		HandleNewSubject(Ar, SubjectKey, SubjectRole, SubjectFrame);
	}

	//End archive with Frame data
	Ar << SubjectFrame.FrameData;
	
	UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, VeryVerbose, "HandleFrame SubjectName=%ls FrameType=%d", *SubjectKey.SubjectName.ToString(), static_cast<uint8>(FrameType));
}

void FNDisplayLiveLinkSubjectReplicator::HandleNewSubject(FArchive& Ar, FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole>& SubjectRole, FLiveLinkSubjectFrameData& SubjectFrame)
{
	//New or updated subjects will have static data 
	Ar << SubjectFrame.StaticData;

	UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "HandleNewSubject SubjectName=%ls", *SubjectKey.SubjectName.ToString());

	//Verify if we're already tracking that subject
	UNDisplayAgentVirtualSubject* TrackedSubject = nullptr;
	if (auto* FoundSujectDataPtr = TrackedSubjects.FindByPredicate(
		[SubjectKey](const UNDisplayAgentVirtualSubject* Other) { return Other->GetAssociatedSubjectKey().SubjectName == SubjectKey.SubjectName; }))
	{
		TrackedSubject = *FoundSujectDataPtr;
		UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "HandleNewSubject SubjectName=%ls but was already tracked. Updating its info.", *SubjectKey.SubjectName.ToString());
	}

	if (Ar.IsSaving())
	{
		FString RoleClassName = SubjectRole->GetPathName();
		Ar << RoleClassName;

		//If we're not tracking that subject yet, create it
		if (TrackedSubject == nullptr)
		{
			TrackedSubject = NewObject<UNDisplayAgentVirtualSubject>(GetTransientPackage());
			TrackedSubjects.Add(TrackedSubject);
		}
		
		//When handling new subject, always setup role and static data
		TrackedSubject->SetTrackedSubjectInfo(SubjectKey, SubjectRole);
		TrackedSubject->GetStaticData().InitializeWith(SubjectFrame.StaticData);
	}
	else
	{
		FString RoleClassName;
		Ar << RoleClassName;
		SubjectRole = StaticLoadClass(UObject::StaticClass(), NULL, *RoleClassName, NULL, LOAD_None, NULL);

		// In a multi-platform environment, a Live Link subject may not be available on all platforms.
		// In such cases, the SubjectRole is invalid. Ignore the subject and do not add it to the tracked subjects.
		if (!IsValid(SubjectRole))
		{
			UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Warning, "Unknown subject role class: %ls", *RoleClassName);
			return;
		}

		//On Agents, create the virtual subject directly in LiveLink so it's part of the system
		ULiveLinkVirtualSubject* ReplicatedSubject = Cast<ULiveLinkVirtualSubject>(LiveLinkClient->GetSubjectSettings(SubjectKey));

		//Always disable the subject we're replicating
		LiveLinkClient->SetSubjectEnabled(SubjectKey, false);

		//If we're not tracking that subject yet, create it
		if (TrackedSubject == nullptr)
		{
			const FLiveLinkSubjectKey ReplicatedSubjectKey(LiveLinkSourceGuid, SubjectKey.SubjectName);
			if (LiveLinkClient->AddVirtualSubject(ReplicatedSubjectKey, UNDisplayAgentVirtualSubject::StaticClass()))
			{
				//Retrieve the newly created subject to add it to our tracking array
				const bool bIncludeDisabled = false;
				const bool bIncludeVirtual = true;
				for (const FLiveLinkSubjectKey& FoundSubjectKey : LiveLinkClient->GetSubjects(bIncludeDisabled, bIncludeVirtual))
				{
					if (FoundSubjectKey.SubjectName == SubjectKey.SubjectName)
					{
						//If this subject has the same subject name, verify if it's the VirtualOne. Settings object will return the VirtualSubject directly
						if (UNDisplayAgentVirtualSubject* NewlyCreatedSubject = Cast<UNDisplayAgentVirtualSubject>(LiveLinkClient->GetSubjectSettings(FoundSubjectKey)))
						{
							TrackedSubject = NewlyCreatedSubject;
							TrackedSubjects.Add(NewlyCreatedSubject);
						}
					}
				}

				checkf(TrackedSubject, TEXT("TrackedSubject '%s' was added but could not be found afterwards."), *SubjectKey.SubjectName.ToString());
			}
			else
			{
				UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Warning, "Could not create new subject '%ls'. It won't be tracked correctly on this node.", *SubjectKey.SubjectName.ToString());
			}
		}

		if (TrackedSubject)
		{
			//VirtualSubjects need translators to be able to be evaluated in different output format. Special treatment required for virtual subjects since they won't/can't exist in the client
			if (ReplicatedSubject)
			{
				const TArray<ULiveLinkFrameTranslator*>& Translators = ReplicatedSubject->GetTranslators();
				TrackedSubject->UpdateTranslators(Translators);
			}
			else
			{
				if (ULiveLinkSubjectSettings* SubjectSetting = Cast<ULiveLinkSubjectSettings>(LiveLinkClient->GetSubjectSettings(SubjectKey)))
				{
					TrackedSubject->UpdateTranslators(SubjectSetting->Translators);
				}
				else
				{
					UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Warning, "Replicating subject '%ls' but could not find its settings. Translators won't work.", *SubjectKey.SubjectName.ToString());
				}
			}

			//When handling new subject, always setup role and static data
			TrackedSubject->SetTrackedSubjectInfo(SubjectKey, SubjectRole);
			TrackedSubject->GetStaticData().InitializeWith(SubjectFrame.StaticData);
		}
	}
}

void FNDisplayLiveLinkSubjectReplicator::ProcessLiveLinkData_Agent(EFrameType FrameType, const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct& FrameData)
{
	//Find associated tracked subject data to broadcast role and static data. Should always be found
	auto* FoundTrackedSubjectPtr = TrackedSubjects.FindByPredicate([SubjectKey](const UNDisplayAgentVirtualSubject* Other) { return Other->GetAssociatedSubjectKey() == SubjectKey; });

	// In a multi-platform environment, a Live Link subject may not be available on all platforms. Ignore if it's not tracked.
	if (!FoundTrackedSubjectPtr)
	{
		return;
	}

	UNDisplayAgentVirtualSubject* FoundTrackedSubject = *FoundTrackedSubjectPtr;
	
	FrameData.GetBaseData()->WorldTime = FPlatformTime::Seconds();
	
	UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, VeryVerbose, "ProcessLiveLinkData SubjectUpdated=%ls UpdateType=%d Timecode=%ls"
		, *(SubjectKey.SubjectName.ToString()), static_cast<uint8>(FrameType)
		, *(FTimecode::FromFrameNumber(FrameData.GetBaseData()->MetaData.SceneTime.Time.FrameNumber, FrameData.GetBaseData()->MetaData.SceneTime.Rate, false).ToString()));
	
	FoundTrackedSubject->UpdateFrameData(MoveTemp(FrameData));
}

void FNDisplayLiveLinkSubjectReplicator::ReInitializeVirtualSource()
{
	//if we are rebuilding our source, let go of any subject we are currently tracking
	for (const UNDisplayAgentVirtualSubject* Subject : TrackedSubjects)
	{
		LiveLinkClient->RemoveVirtualSubject(Subject->GetSubjectKey());
	}

	//Start source with a clean slate
	TrackedSubjects.Empty();

	//Create our Source to hold all VirtualSubject we'll create
	LiveLinkSourceGuid = LiveLinkClient->AddVirtualSubjectSource(TEXT("nDisplaySubjectReplicator"));
	if (!LiveLinkSourceGuid.IsValid())
	{
		UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Error, "Could not add VirtualSubject Source for nDisplay replication.");
	}
}

void FNDisplayLiveLinkSubjectReplicator::OnLiveLinkSourceRemoved(FGuid SourceGuid)
{
	if (LiveLinkSourceGuid.IsValid() && LiveLinkSourceGuid == SourceGuid)
	{
		UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "nDisplay LiveLink Source was removed. Reinitializing ourself.");
		ReInitializeVirtualSource();
	}
}

void FNDisplayLiveLinkSubjectReplicator::OnModularFeatureRemoved(const FName& Type, IModularFeature* ModularFeature)
{
	//If LiveLink gets unregistered, invalidate our cached client which make the replicator do nothing
	//To support hot reload, we would need to register for the feature to be loaded again
	if (Type == ILiveLinkClient::ModularFeatureName)
	{
		UE_LOGF(LogNDisplayLiveLinkSubjectReplicator, Verbose, "LiveLink feature was removed.");
		LiveLinkClient = nullptr;
	}
}

void FNDisplayLiveLinkSubjectReplicator::OnDisplayClusterPrimaryNodeChanged(const FString& NewPrimaryId)
{
	const FString ThisNodeId = IDisplayCluster::Get().GetClusterMgr()->GetNodeId();
	if (ThisNodeId.Equals(NewPrimaryId, ESearchCase::IgnoreCase))
	{
		bBecamePrimary = true;
	}
}

void FNDisplayLiveLinkSubjectReplicator::SetSubjectsEnabled(bool bIncludeDisabled, bool bIncludeVirtual, bool bEnabled)
{
	const TArray<FLiveLinkSubjectKey> Subjects = LiveLinkClient->GetSubjects(bIncludeDisabled, bIncludeVirtual);
	for (const FLiveLinkSubjectKey& Subject : Subjects)
	{
		if (Subject.Source != LiveLinkSourceGuid)
		{
			LiveLinkClient->SetSubjectEnabled(Subject, bEnabled);
		}
	}
}
