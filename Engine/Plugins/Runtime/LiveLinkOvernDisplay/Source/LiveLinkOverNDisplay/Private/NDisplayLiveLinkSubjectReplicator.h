// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IDisplayClusterClusterSyncObject.h"
#include "UObject/GCObject.h"

#include "LiveLinkTypes.h"

#include <atomic>

class ULiveLinkRole;
template <typename T> class TSubclassOf;


class ILiveLinkClient;
class IModularFeature;
class UNDisplayAgentVirtualSubject;


/**
 * Classes used to replicates data to be used for each frame for each enabled subjects on the Controller machines.
 * Agents will use that replicated data to use the same thing on each machines
 */
class LIVELINKOVERNDISPLAY_API FNDisplayLiveLinkSubjectReplicator : public IDisplayClusterClusterSyncObject, public FGCObject
{
private:
	enum class EFrameType : uint8
	{
		DataOnly,
		NewSubject,			//A new subject was sent this frame.
		UpdatedSubject,		//The subject's static data or role was updated this frame.
	};

public:

	FNDisplayLiveLinkSubjectReplicator() = default;
	virtual ~FNDisplayLiveLinkSubjectReplicator();

	/** Move only type since we're owning LiveLink base structs */
	FNDisplayLiveLinkSubjectReplicator(const FNDisplayLiveLinkSubjectReplicator&) = delete;
	FNDisplayLiveLinkSubjectReplicator& operator=(const FNDisplayLiveLinkSubjectReplicator&) = delete;

	//~ Begin IDisplayClusterClusterSyncObject interface
	virtual bool IsActive() const override;
	virtual FString GetSyncId() const override;
	virtual bool IsDirty() const override { return true; };
	virtual void SerializeDC(FArchive& Ar) override;
	//~ End IDisplayClusterClusterSyncObject interface

	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FNDisplayLiveLinkSubjectReplicator");
	}
	//~ End FGCObject interface

	/** Initializes all required callbacks and LiveLink source */
	void Initialize();
	
	/** Releases all internal resources and callbacks */
	void Release();

	/** Registers ourself as sync object in nDisplay */
	void Activate();
	
	/** Unregisters ourself as sync object from nDisplay */
	void Deactivate();

protected:

	/**
	 * Hook to beginning of frames to make sure our replicated virtual subjects are the ones enabled
	 * @note Only called on Agent
	 */
	void OnEngineBeginFrame();

	/** Synchronization point when object is going to be synchronized across nDisplay cluster */
	void OnDataSynchronization(FArchive& Ar);

	/** Process new subject data when serializing SyncObject */
	void HandleNewSubject(FArchive& Ar, FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole>& SubjectRole, FLiveLinkSubjectFrameData& SubjectFrame);

	/** Process a frame for a specific subject */
	void HandleFrame(FArchive& Ar, EFrameType& FrameType, FLiveLinkSubjectKey& SubjectKey, TSubclassOf<ULiveLinkRole>& SubjectRole, FLiveLinkSubjectFrameData& SubjectFrame);

	/** Only used on agents, handles this frame data for a subject. If a new subject is handled, the associated VirtualSubject will be created */
	void ProcessLiveLinkData_Agent(EFrameType FrameType, const FLiveLinkSubjectKey& SubjectKey, FLiveLinkFrameDataStruct& FrameData);

	/** For agent only, remove tracked subjets, add ourself as a source  */
	void ReInitializeVirtualSource();

	/** If our source is removed (could happen if a new preset is applied), reinitialize ourself to stay awake */
	void OnLiveLinkSourceRemoved(FGuid SourceGuid);

	/** Listen for modular feature removed in case LiveLink gets unloaded */
	void OnModularFeatureRemoved(const FName& Type , IModularFeature* ModularFeature);

	/** Post-failover handler. It's a callback that informs about new p-node ID */
	void OnDisplayClusterPrimaryNodeChanged(const FString& NewPrimaryId);

	/** A helper to enable/disable all the subjects */
	void SetSubjectsEnabled(bool bIncludeDisabled, bool bIncludeVirtual, bool bEnabled);

private:

	/** Cached LiveLinkClient when modular feature is registered */
	ILiveLinkClient* LiveLinkClient = nullptr;

	/** List of Subjects that we are replicating across cluster. On Agents, it will actually be added to the LiveLink subject's list */
	TArray<TObjectPtr<UNDisplayAgentVirtualSubject>> TrackedSubjects;

	/** Guid associated to our Virtual Subject Source */
	FGuid LiveLinkSourceGuid;

	/** Used to re-initialize current node after failure handling */
	std::atomic<bool> bBecamePrimary = false;
};
