// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FNDisplayLiveLinkSubjectReplicator;

/**
 * Interface for LiveLinkOverNDisplay module.
 */
class ILiveLinkOverNDisplayModule : public IModuleInterface
{
public:

	/**
	 * Gets a reference to the LiveLinkOverNDisplay module instance.
	 */
	static ILiveLinkOverNDisplayModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILiveLinkOverNDisplayModule>("LiveLinkOverNDisplay");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("LiveLinkOverNDisplay");
	}

	virtual FNDisplayLiveLinkSubjectReplicator& GetSubjectReplicator() = 0;

public:

	/** Virtual destructor. */
	virtual ~ILiveLinkOverNDisplayModule() { }
};

