// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "LiveLinkOverNDisplaySettings.generated.h"

UCLASS(config=Engine)
class LIVELINKOVERNDISPLAY_API ULiveLinkOverNDisplaySettings : public UObject
{
	GENERATED_BODY()

public:
	ULiveLinkOverNDisplaySettings();

	/** Whether or not LiveLink over NDisplay is enabled, from command line first or from project settings */
	bool IsLiveLinkOverNDisplayEnabled() const { return bIsEnabledFromCommandLine.IsSet() ? bIsEnabledFromCommandLine.GetValue() : bIsEnabled; }

protected:
	/** 
	 * Enables nDisplay specific LiveLink subjects management across nDisplay cluster. 
	 * @note Can be overrided via the command line using -EnableLiveLinkOverNDisplay=false
	 */
	UPROPERTY(config, EditAnywhere, Category = "LiveLinkOverNDisplay Settings")
	bool bIsEnabled = true;

	/** Command line override value if it was set on the command line */
	TOptional<bool> bIsEnabledFromCommandLine;
};

